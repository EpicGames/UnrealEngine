// Copyright Epic Games, Inc. All Rights Reserved.

#include "FastXml.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/FileHelper.h"
#include "Misc/FeedbackContext.h"

#define LOCTEXT_NAMESPACE "FastXml"


/**
 * Implementation of fast XML parser
 */
class FFastXmlImpl
{

public:

	FFastXmlImpl( IFastXmlCallback* InitCallback, TCHAR* InitXmlFileContents, const SIZE_T InitXmlFileContentsLength, FFeedbackContext* InitFeedbackContext )
		: Callback( InitCallback ),
		  XmlFileContents( InitXmlFileContents ),
		  XmlFileContentsLength( InitXmlFileContentsLength ),
		  FeedbackContext( InitFeedbackContext )
	{
		// Setup character type map.  It maps each ASCII character to the type of XML character it is.
		{
			memset( CharacterTypeMap, (int)ECharType::Data, sizeof( CharacterTypeMap ) );
			CharacterTypeMap[ 0 ] = ECharType::EndOfFile;
			CharacterTypeMap[ L' ' ] = CharacterTypeMap[ L'\t' ] = ECharType::Whitespace;
			CharacterTypeMap[ L'/' ] = CharacterTypeMap[ L'>' ] = CharacterTypeMap[ L'?' ] = ECharType::EndOfElement;
			CharacterTypeMap[ L'\n' ] = CharacterTypeMap[ L'\r' ] = ECharType::EndOfLine;
		}

		StackIndex = 0;
		for( uint32 StackLevel = 0; StackLevel < ( MaxStackDepth + 1 ); StackLevel++ )
		{
			Stack[ StackLevel ] = nullptr;
			IsStackAllocated[ StackLevel ] = false;
		}
	}


	~FFastXmlImpl( void )
	{
		StackIndex = 0;

		for( uint32 StackLevel = 0; StackLevel < ( StackIndex + 1 ); StackLevel++ )
		{
			if( IsStackAllocated[ StackLevel ] )
			{
				FMemory::Free( (void*)Stack[ StackLevel ] );
				IsStackAllocated[ StackLevel ] = false;
			}
			Stack[ StackLevel ] = nullptr;
		}
	}


	bool ProcessXmlFile( FText& OutErrorMessage, int32& OutErrorLineNumber )
	{
		// Parse the file!
		bool bResult = ProcessXmlFileInternal() && ( FeedbackContext == nullptr || !FeedbackContext->ReceivedUserCancel() );

		// Keep track of any errors for the caller
		OutErrorMessage = ErrorMessage;
		OutErrorLineNumber = LineNumber;

		return bResult;
	}


protected:


	TCHAR* ProcessClose( const TCHAR Char, const TCHAR* Element, TCHAR* Buffer, const int32 AttributeCount, const TCHAR** Attributes )
	{
		if( FeedbackContext != nullptr )
		{
			FeedbackContext->UpdateProgress(UE_PTRDIFF_TO_INT32(Buffer - &XmlFileContents[0]), IntCastChecked<int32>(XmlFileContentsLength) );
		}

		if( Char == L'/' || Char == L'?' )
		{
			TCHAR* Slash = (TCHAR*)FCString::Strchr( Element, Char );
			if( Slash )
			{
				*Slash = 0;
			}

			if( Char == L'?' && FCString::Strcmp( Element, TEXT( "xml" ) ) == 0 )
			{
				if( !Callback->ProcessXmlDeclaration( 0, LineNumber ) )
				{
					return nullptr;
				}
			}
			else
			{
				if( !Callback->ProcessElement( Element, 0, LineNumber ) )
				{
					ErrorMessage = LOCTEXT( "UserAbortedOnElement", "User aborted the parsing process" );
					return nullptr;
				}

				check( AttributeCount % 2 == 0 );
				for( int32 AttributeIndex = 0; AttributeIndex < AttributeCount / 2; ++AttributeIndex )
				{
					if( !Callback->ProcessAttribute( Attributes[ AttributeIndex * 2 ], Attributes[ AttributeIndex * 2 + 1 ] ) )
					{
						ErrorMessage = LOCTEXT( "UserAbortedOnAttribute", "User aborted the parsing process" );
						return nullptr;
					}
				}

				PushElement( Element );

				const TCHAR* Close = PopElement();
				if( !Callback->ProcessClose( Close ) )
				{
					return nullptr;
				}
			}

			if( !Slash )
			{
				++Buffer;
			}
		}
		else
		{
			Buffer = SkipNextData( Buffer );
			TCHAR* Data = Buffer; // this is the data portion of the element, only copies memory if we encounter line feeds
			TCHAR* DestData = 0;
			while( *Buffer && *Buffer != L'<' )
			{
				if( IsCharTypeEqual(*Buffer, ECharType::EndOfLine) )
				{
					if( *Buffer == L'\r' )
					{
						LineNumber++;
					}
					DestData = Buffer;
					*DestData++ = L' '; // replace the linefeed with a space...
					Buffer = SkipNextData( Buffer );
					while( *Buffer && *Buffer != L'<' )
					{
						if( IsCharTypeEqual(*Buffer, ECharType::EndOfLine) )
						{
							if( *Buffer == L'\r' )
							{
								LineNumber++;
							}
							*DestData++ = L' '; // replace the linefeed with a space...
							Buffer = SkipNextData( Buffer );
						}
						else
						{
							*DestData++ = *Buffer++;
						}
					}
					break;
				}
				else
				{
					++Buffer;
				}
			}

			if( *Buffer == L'<' )
			{
				if( DestData )
				{
					*DestData = 0;
				}
				else
				{
					*Buffer = 0;
				}

				Buffer++; // skip it..

				if( *Data == 0 )
				{
					Data = 0;
				}

				if( !Callback->ProcessElement( Element, Data, LineNumber ) )
				{
					ErrorMessage = LOCTEXT( "UserAbortedOnElement", "User aborted the parsing process" );
					return nullptr;
				}

				check( AttributeCount % 2 == 0 );
				for( int32 AttributeIndex = 0; AttributeIndex < AttributeCount / 2; ++AttributeIndex )
				{
					if( !Callback->ProcessAttribute( Attributes[ AttributeIndex * 2 ], Attributes[ AttributeIndex * 2 + 1 ] ) )
					{
						ErrorMessage = LOCTEXT( "UserAbortedOnAttribute", "User aborted the parsing process" );
						return nullptr;
					}
				}

				PushElement( Element );

				// check for the comment use case...
				if( Buffer[ 0 ] == L'!' && Buffer[ 1 ] == L'-' && Buffer[ 2 ] == L'-' )
				{
					Buffer += 3;
					while( *Buffer && *Buffer == L' ' )
					{
						++Buffer;
					}

					TCHAR* Comment = Buffer;
					TCHAR* CommentEnd = FCString::Strstr( Buffer, TEXT( "-->" ) );
					if( CommentEnd )
					{
						*CommentEnd = 0;
						Buffer = CommentEnd + 3;
						if( !Callback->ProcessComment( Comment ) )
						{
							ErrorMessage = LOCTEXT( "UserAbortedOnComment", "User aborted the parsing process" );
							return nullptr;
						}
					}
				}
				else if( *Buffer == L'/' )
				{
					Buffer = ProcessClose( Buffer );
				}
			}
			else
			{
				ErrorMessage = LOCTEXT( "ElementDataNotTerminated", "Data portion of an element wasn't terminated properly" );
				return nullptr;
			}
		}

		if( FeedbackContext != nullptr && FeedbackContext->ReceivedUserCancel() )
		{
			Buffer = nullptr;
			ErrorMessage = LOCTEXT( "UserAbortedOnFile", "User cancelled processing of this file" );
		}

		return Buffer;
	}


	TCHAR* ProcessClose( TCHAR* Buffer )
	{
		const TCHAR* Start = PopElement();
		const TCHAR* Close = Start;
		if( Buffer[ 1 ] != L'>' )
		{
			Buffer++;
			Close = Buffer;
			
			while( *Buffer && *Buffer != L'>' )
			{
				Buffer++;
			}

			*Buffer = 0;
		}

		if( 0 != FCString::Strcmp( Start, Close ) )
		{
			ErrorMessage = LOCTEXT( "OpenCloseTagsNotMatched", "Open and closing tags do not match" );
			return nullptr;
		}

		if( !Callback->ProcessClose( Close ) )
		{
			return nullptr;
		}
		++Buffer;

		return Buffer;
	}


	bool ProcessXmlFileInternal()
	{
		LineNumber = 1;

		for( uint32 StackLevel = 0; StackLevel < ( StackIndex + 1 ); StackLevel++ )
		{
			if( !IsStackAllocated[ StackLevel ] )
			{
				const TCHAR* Text = Stack[ StackLevel ];
				if( Text )
				{
					uint32 TextLength = (uint32)FCString::Strlen( Text );
					Stack[ StackLevel ] = (const TCHAR*)FMemory::Malloc( TextLength * sizeof( TCHAR ) + sizeof( TCHAR ) );
					memcpy( (void*)Stack[ StackLevel ], Text, TextLength * sizeof( TCHAR ) + sizeof( TCHAR ) );
					IsStackAllocated[ StackLevel ] = true;
				}
			}
		}

		TCHAR* Element;
		TCHAR* Buffer = &XmlFileContents[ 0 ];
		TCHAR* AttributeDelimiter;

		while( *Buffer )
		{
			Buffer = SkipNextData( Buffer );
			if( *Buffer == 0 )
			{
				break;
			}

			if( *Buffer == L'<' )
			{
				Buffer++;
				if( *Buffer == L'?' ) // Allow XML declarations
				{
					Buffer++;
				}
				else if( Buffer[ 0 ] == L'!' && Buffer[ 1 ] == L'-' && Buffer[ 2 ] == L'-' )
				{
					Buffer += 3;
					while( *Buffer && *Buffer == L' ' )
					{
						Buffer++;
					}

					TCHAR* Comment = Buffer;
					TCHAR* CommentEnd = FCString::Strstr( Buffer, TEXT( "-->" ) );
					if( CommentEnd )
					{
						*CommentEnd = 0;
						Buffer = CommentEnd + 3;
						if( !Callback->ProcessComment( Comment ) )
						{
							ErrorMessage = LOCTEXT( "UserAbortedOnComment", "User aborted the parsing process" );
							return false;
						}
					}
					continue;
				}
			}

			if( *Buffer == L'/' )
			{
				Buffer = ProcessClose( Buffer );
				if( !Buffer )
				{
					return false;
				}
			}
			else
			{
				if( *Buffer == L'?' )
				{
					Buffer++;
				}

				Element = Buffer;
				int32 AttributeCount = 0;
				const TCHAR* Attributes[ MaxAttributes ];
				bool Close;
				Buffer = NextWhitespaceOrClose( Buffer, Close );
				if( Close )
				{
					TCHAR Char = *( Buffer - 1 );
					if( Char != L'?' && Char != L'/' )
					{
						Char = L'>';
					}
					*Buffer++ = 0;
					Buffer = ProcessClose( Char, Element, Buffer, AttributeCount, Attributes );
					if( !Buffer )
					{
						return false;
					}
				}
				else
				{
					if( *Buffer == 0 )
					{
						return true;
					}

					*Buffer = 0; // place a zero byte to indicate the end of the element name...
					Buffer++;

					while( *Buffer )
					{
						Buffer = SkipNextData( Buffer ); // advance past any soft seperators (tab or space)

						if( IsCharTypeEqual(*Buffer, ECharType::EndOfElement) )
						{
							TCHAR Char = *Buffer++;
							if( L'?' == Char )
							{
								if( L'>' != *Buffer ) //?>
								{
									check( 0 );
									return false;
								}

								Buffer++;
							}
							Buffer = ProcessClose( Char, Element, Buffer, AttributeCount, Attributes );
							if( !Buffer )
							{
								return false;
							}
							break;
						}
						else
						{
							if( AttributeCount >= MaxAttributes )
							{
								ErrorMessage = LOCTEXT( "TooManyAttributes", "Encountered too many attributes in a single element for this parser to handle" );
								return false;
							}
							check(AttributeCount >= 0);
							Attributes[ AttributeCount ] = Buffer;
							Buffer = NextSeparator( Buffer );  // scan up to a space, or an equal
							if( *Buffer )
							{
								if( *Buffer != L'=' )
								{
									*Buffer = 0;
									Buffer++;
									while( *Buffer && *Buffer != L'=' )
									{
										Buffer++;
									}

									if( *Buffer == L'=' )
									{
										Buffer++;
									}
								}
								else
								{
									*Buffer = 0;
									Buffer++;
								}

								if( *Buffer ) // if not eof...
								{
									Buffer = SkipNextData( Buffer );
									if( *Buffer == L'"' || *Buffer == L'\'' )
									{
										AttributeDelimiter = Buffer;
										Buffer++;
										AttributeCount++;
										Attributes[ AttributeCount ] = Buffer;
										AttributeCount++;
										while(*Buffer && *Buffer != *AttributeDelimiter)
										{
											Buffer++;
										}
										if(*Buffer == *AttributeDelimiter)
										{
											*Buffer = 0;
											Buffer++;
										}
										else
										{
											ErrorMessage = LOCTEXT( "NoClosingQuoteForAttribute", "Failed to find closing quote for attribute" );
											return false;
										}
									}
									else
									{
										// Missing quote after attribute.  We'll handle it as best we can.
										AttributeCount--;
										while( *Buffer != L'/' && *Buffer != L'>' && *Buffer != 0 )
										{
											Buffer++;
										}
									}
								}
							}
						}
					}
				}
			}
		}

		if( StackIndex )
		{
			ErrorMessage = LOCTEXT( "InvalidFileFormat", "Invalid file format" );
			return false;
		}

		return true;
	}

	
	inline TCHAR* NextWhitespace( TCHAR* Buffer )
	{
		while( *Buffer && !IsCharTypeEqual(*Buffer, ECharType::Whitespace))
		{
			Buffer++;
		}

		return Buffer;
	}


	inline TCHAR* NextWhitespaceOrClose( TCHAR* Buffer, bool& IsClose )
	{
		while( *Buffer && !IsCharTypeEqual(*Buffer, ECharType::Whitespace) && *Buffer != L'>' )
		{
			Buffer++;
		}

		IsClose = *Buffer == L'>';
		return Buffer;
	}


	inline TCHAR* NextSeparator( TCHAR* Buffer )
	{
		while( *Buffer && !IsCharTypeEqual(*Buffer, ECharType::Whitespace) && *Buffer != L'=' )
		{
			Buffer++;
		}

		return Buffer;
	}


	inline TCHAR* SkipNextData( TCHAR* Buffer )
	{
		// while we have data, and we encounter soft seperators or line feeds...
		while( *Buffer && ( IsCharTypeEqual(*Buffer, ECharType::Whitespace) || IsCharTypeEqual(*Buffer, ECharType::EndOfLine) ) )
		{
			if( *Buffer == L'\n' )
			{
				LineNumber++;
			}

			Buffer++;
		}
		return Buffer;
	}


	void PushElement( const TCHAR* Element )
	{
		check( StackIndex < MaxStackDepth );
		if( StackIndex < MaxStackDepth )
		{
			if( IsStackAllocated[ StackIndex ] )
			{
				FMemory::Free( (void*)Stack[ StackIndex ] );
			}
			IsStackAllocated[ StackIndex ] = false;
			Stack[ StackIndex++ ] = Element;
		}
	}

	const TCHAR* PopElement( void )
	{
		if( IsStackAllocated[ StackIndex ] )
		{
			FMemory::Free( (void*)Stack[ StackIndex ] );
			Stack[ StackIndex ] = nullptr;
			IsStackAllocated[ StackIndex ] = false;
		}
		return StackIndex ? Stack[ --StackIndex ] : nullptr;
	}


	/** User callback to report XML data and progress */
	IFastXmlCallback* Callback;

	/** Reference to the the contents of the XML file to parse */
	TCHAR* XmlFileContents;

	/** Length of the XmlFileContents string */
	SIZE_T XmlFileContentsLength;

	/** Feedback context for status reporting.  Can be nullptr. */
	FFeedbackContext* FeedbackContext;

	/** If anything goes wrong or the user cancels, the error message to return will be stored here */
	FText ErrorMessage;

	/** Types of characters we'll encounter while parsing */
	enum class ECharType : uint8
	{
		Data = 0,
		EndOfFile,
		Whitespace,
		EndOfElement, // Either a forward slash or a greater than symbol
		EndOfLine,
	};

	inline bool IsCharTypeEqual( TCHAR Character, ECharType Type )
	{
		if (Character < 256)
		{
			return CharacterTypeMap[Character] == Type;
		}
		return Type == ECharType::Data;
	}

	/** Maps each ASCII character to the type of character we think it is */
	ECharType CharacterTypeMap[ 256 ];

	/** Current stack depth as we descend through XML nodes and attributes */
	uint32 StackIndex;

	/** The current line number we're on in the file */
	int32 LineNumber;

	/** Maximum stack depth.  We can't support XML documents that go deeper than this! */
	static const int MaxStackDepth = 2048;

	/** Maximum number of attributes in an element that we can support */
	static const int MaxAttributes = 2048;

	/** Stack of characters, one for each stack depth */
	const TCHAR* Stack[ MaxStackDepth + 1 ];

	/** For each stack level, whether we've allocated that stack or not yet */
	bool IsStackAllocated[ MaxStackDepth + 1 ];
};



bool FFastXml::ParseXmlFile( IFastXmlCallback* Callback, const TCHAR* XmlFilePath, TCHAR* XmlFileContents, FFeedbackContext* FeedbackContext, const bool bShowSlowTaskDialog, const bool bShowCancelButton, FText& OutErrorMessage, int32& OutErrorLineNumber )
{
	bool bSuccess = true;

	FString LoadedXmlFileContents;
	SIZE_T XmlFileContentsLength = 0;
	if( XmlFilePath != nullptr && FPlatformString::Strlen( XmlFilePath ) > 0 )
	{
		if( FPlatformFileManager::Get().GetPlatformFile().FileExists( XmlFilePath ) )
		{
			if( FeedbackContext != nullptr )
			{
				FeedbackContext->BeginSlowTask( LOCTEXT( "LoadingXML", "Loading XML file..." ), bShowSlowTaskDialog, false /* Cannot support cancelling the loading part */ );
			}

			if( FFileHelper::LoadFileToString( LoadedXmlFileContents, XmlFilePath ) )
			{
				XmlFileContentsLength = LoadedXmlFileContents.Len();
				if( XmlFileContentsLength > 0 )
				{
					// File was loaded okay!
					XmlFileContents = &LoadedXmlFileContents[ 0 ];
				}
				else
				{ 
					bSuccess = false;
					OutErrorMessage = LOCTEXT( "LoadedXMLFileWasEmpty", "The XML file is empty" );
					OutErrorLineNumber = 1;
				}
			}
			else
			{
				bSuccess = false;
				OutErrorMessage = LOCTEXT( "ErrorReadingFile", "Unable to load the XML file" );
				OutErrorLineNumber = 1;
			}

			if( FeedbackContext != nullptr )
			{
				FeedbackContext->EndSlowTask();
			}
		}
		else
		{
			bSuccess = false;
			OutErrorMessage = LOCTEXT( "FileNotFound", "Couldn't find the specified XML file on disk" );
			OutErrorLineNumber = 1;
		}
	}
	else
	{
		if( ensure( XmlFileContents != nullptr && *XmlFileContents != 0 ) )
		{
			XmlFileContentsLength = FPlatformString::Strlen( XmlFileContents );
		}
		else
		{
			bSuccess = false;
			OutErrorMessage = LOCTEXT( "NoFileNameOrContentsPassedIn", "ParseXmlFile() was called without either an XML file name or an XML file contents text buffer supplied.  Either XmlFilePath or XmlFileContents must be valid in order to call ParseXmlFile()" );
			OutErrorLineNumber = 1;
		}
	}

	if( bSuccess )
	{
		if( FeedbackContext != nullptr )
		{
			FeedbackContext->BeginSlowTask( LOCTEXT( "ProcessingXML", "Processing XML file..." ), bShowSlowTaskDialog, bShowCancelButton );
		}

		// Parse the XML file contents!
		TUniquePtr< FFastXmlImpl > FastXmlImpl( new FFastXmlImpl( Callback, XmlFileContents, XmlFileContentsLength, FeedbackContext ) );
		bSuccess = FastXmlImpl->ProcessXmlFile( OutErrorMessage, OutErrorLineNumber );

		if( FeedbackContext != nullptr )
		{
			FeedbackContext->EndSlowTask();
		}
	}

	return bSuccess;
}


#undef LOCTEXT_NAMESPACE
