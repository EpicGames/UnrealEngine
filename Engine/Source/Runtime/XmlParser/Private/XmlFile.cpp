// Copyright Epic Games, Inc. All Rights Reserved.

#include "XmlFile.h"
#include "Algo/AllOf.h"

#include "Misc/FileHelper.h"
#include "Misc/ScopeExit.h"
FXmlFile::FXmlFile(const FString& InFile, EConstructMethod::Type ConstructMethod)
	: RootNode(nullptr), bFileLoaded(false)
{
	LoadFile(InFile, ConstructMethod);
}

bool FXmlFile::LoadFile(const FString& InFile, EConstructMethod::Type ConstructMethod)
{
	// Remove any file stuff if it already exists
	Clear();

	// So far no error (Early so it can be overwritten below by errors)
	ErrorMessage = NSLOCTEXT("XmlParser", "LoadSuccess", "XmlFile was loaded successfully").ToString();

	TArray<FString> Input;
	if(ConstructMethod == EConstructMethod::ConstructFromFile)
	{
		// Read and split the file
		if (!FFileHelper::LoadFileToStringArray(Input, *InFile))
		{
			ErrorMessage = NSLOCTEXT("XmlParser", "FileLoadFail", "Failed to load the file").ToString();
			ErrorMessage += TEXT("\"");
			ErrorMessage += InFile;
			ErrorMessage += TEXT("\"");
			return false;
		}
	}
	else
	{
		// Parse input buffer into an array of lines
		InFile.ParseIntoArrayLines(Input, /*bCullEmpty*/false);
	}

	// Pre-process the input
	PreProcessInput(Input);

	if (bHackAttemptToPreserveWhitespace)
	{
		// Reintroduce line endings which are stripped by ParseIntoArrayLines / LoadFileToStringArray.
		for (FString& Line : Input)
		{
			Line.AppendChar('\n');
		}
	}

	// Tokenize the input
	TArray<FString> Tokens = Tokenize(Input);

	// Parse the input & create the nodes
	CreateNodes(Tokens);

	// All done with creation, set up necessary information
	if(bFileLoaded == true)
	{
		if(ConstructMethod == EConstructMethod::ConstructFromFile)
		{
			LoadedFile = InFile;
		}
	}
	else
	{
		LoadedFile = TEXT("");
		RootNode = nullptr;
	}

	// Now check the status flag of the creation. It may have actually failed, but given us a 
	// partially created representation
	if(bCreationFailed)
	{
		Clear();
	}

	return bFileLoaded;
}

FString FXmlFile::GetLastError() const
{
	return ErrorMessage;
}

void FXmlFile::Clear()
{
	if(bFileLoaded)
	{
		check(RootNode != nullptr);
		RootNode->Delete();
		delete RootNode;

		RootNode = nullptr;
		bFileLoaded = false;
		LoadedFile = TEXT("");
		ErrorMessage = NSLOCTEXT("XmlParser", "ClearSuccess", "XmlFile was cleared successfully").ToString();
	}
}

bool FXmlFile::IsValid() const
{
	checkSlow(bFileLoaded ? RootNode != nullptr : RootNode == nullptr);
	return bFileLoaded;
}

const FXmlNode* FXmlFile::GetRootNode() const
{
	return RootNode;
}

FXmlNode* FXmlFile::GetRootNode()
{
	return RootNode;
}

bool FXmlFile::Save(const FString& Path)
{
	FString Xml = TEXT("<?xml version=\"1.0\" encoding=\"UTF-8\"?>") LINE_TERMINATOR;

	const FXmlNode* CurrentNode = GetRootNode();
	if(CurrentNode != nullptr)
	{
		WriteNodeHierarchy(*CurrentNode, FString(), Xml);
	}
	
	if (!FFileHelper::SaveStringToFile(Xml, *Path, FFileHelper::EEncodingOptions::ForceUTF8WithoutBOM))
	{
		ErrorMessage = NSLOCTEXT("XmlParser", "FileSaveFail", "Failed to save the file").ToString();
		ErrorMessage += FString::Printf(TEXT("\"%s\""), *Path);
		return false;
	}

	return true;
}

/** Checks if the passed character is a whitespace character */
static bool IsWhiteSpace(TCHAR Char)
{
	// Whitespace will be any character that is not a common printed ASCII character (and not space/tab)
	if(Char == TCHAR(' ') ||
		Char == TCHAR('\t') ||
		Char < 32 ) // ' '
	{
		return true;
	}
	return false;
}

void FXmlFile::PreProcessInput(TArray<FString>& Input)
{
	// Note: This implementation is written simply and will not handle all cases.  It
	//       is made for the simple cases where FXmlFile is to be used.
	// Assumptions/Misc:
	//       - Well-formatted file with 1 entry per line
	//       - Ignoring versions, encodings, and doctypes

	// Remove white space at the beginning of lines
	for(int32 i = 0; i < Input.Num(); ++i)
	{
		int32 NumWhiteSpace = 0;
		for(int32 j = 0; j < Input[i].Len(); ++j)
		{
			if(!IsWhiteSpace(Input[i][j]))
			{
				break;
			}
			
			++NumWhiteSpace;
		}

		if(NumWhiteSpace > 0)
		{
			Input[i].MidInline(NumWhiteSpace, MAX_int32, EAllowShrinking::No);
		}
	}

	// Cull any text that can be removed on a line-based parse
	for(int32 i = 0; i < Input.Num(); ++i)
	{
		// Find <!DOCTYPE or <?xml and remove those lines
		if(Input[i].StartsWith(TEXT("<!DOCTYPE"), ESearchCase::IgnoreCase) || Input[i].StartsWith(TEXT("<?xml"), ESearchCase::IgnoreCase))
		{
			Input[i] = TEXT("");
		}
	}

	// Cull any text inside of comments
	bool bInComment = false;
	int32 CommentLineStart = -1;
	int32 CommentIndexStart = -1;
	for(int32 i = 0; i < Input.Num(); ++i)
	{
		if(Input[i].Len() == 3)
		{
			if(bInComment)
			{
				if(Input[i][0] == TCHAR('-') && Input[i][1] == TCHAR('-') && Input[i][2] == TCHAR('>'))
				{
					// Found comment end, perform removal (simply replace all text with whitespace to be ignored by tokenizer)
					bInComment = false;
					int32 CommentLineEnd = i;
					int32 CommentIndexEnd = 2;
					WhiteOut(Input, CommentLineStart, CommentLineEnd, CommentIndexStart, CommentIndexEnd);
				}
			}
		}

		if(Input[i].Len() < 3)
		{
			continue;
		}

		int32 Indx1 = 0, Indx2 = 1, Indx3 = 2, Indx4 = 3;
		for(; Indx4 < Input[i].Len(); ++Indx1, ++Indx2, ++Indx3, ++Indx4)
		{
			// Looking for the start of a comment
			if(!bInComment)
			{
				if(Input[i][Indx1] == TCHAR('<') && Input[i][Indx2] == TCHAR('!') && Input[i][Indx3] == TCHAR('-') && Input[i][Indx4] == TCHAR('-'))
				{
					// Found comment start, mark it
					bInComment = true;
					CommentLineStart = i;
					CommentIndexStart = Indx1;
				}
			}

			// Looking for the end of a comment
			else
			{
				if( (Input[i][Indx2] == TCHAR('-') && Input[i][Indx3] == TCHAR('-') && Input[i][Indx4] == TCHAR('>')) ||
					(Input[i][Indx1] == TCHAR('-') && Input[i][Indx2] == TCHAR('-') && Input[i][Indx3] == TCHAR('>')) )
				{
					// Found comment end, perform removal (simply replace all text with whitespace to be ignored by tokenizer)
					bInComment = false;
					int32 CommentLineEnd = i;
					int32 CommentIndexEnd = Indx4;
					WhiteOut(Input, CommentLineStart, CommentLineEnd, CommentIndexStart, CommentIndexEnd);
				}
			}
		}
	}
}

void FXmlFile::WhiteOut(TArray<FString>& Input, int32 LineStart, int32 LineEnd, int32 IndexStart, int32 IndexEnd)
{
	if(LineEnd < LineStart)
	{
		// Error!, malformed file with comment ends
	}

	// White-out first line
	if(LineEnd - LineStart > 0)
	{
		for(int32 i = IndexStart; i < Input[LineStart].Len(); ++i)
		{
			Input[LineStart][i] = TCHAR(' ');
		}
	}

	// White-out middle lines
	if(LineEnd - LineStart > 1)
	{
		int32 NumLines = LineEnd - LineStart - 1;
		int32 MidStart = LineStart + 1;
		for(int32 i = MidStart; i < MidStart + NumLines; ++i)
		{
			Input[i] = TEXT("");
		}
	}

	// White-out last line
	if(LineEnd - LineStart > 0)
	{
		for(int32 i = 0; i <= IndexEnd; ++i)
		{
			Input[LineEnd][i] = TCHAR(' ');
		}
	}

	// White-out if comment is only on 1 line
	if(LineStart == LineEnd)
	{
		for(int32 i = IndexStart; i <= IndexEnd; ++i)
		{
			Input[LineStart][i] = TCHAR(' ');
		}
	}
}

/** Checks if the passed character is an operator */
static bool CheckTagOperator(const TCHAR* PtrStart, const TCHAR* Ptr, const TCHAR* PtrEnd)
{
	checkSlow(PtrStart <= Ptr && Ptr < PtrEnd);

	TCHAR Ch = *Ptr;
	if(Ch == TCHAR('/'))
	{
		const TCHAR* PtrNext = Ptr + 1;
		if(PtrNext != PtrEnd && *PtrNext == TCHAR('>'))
		{
			return true;
		}

		// check either the next or previous chars are tag closures - otherwise, this is just a slash
		if(Ptr != PtrStart  && *(Ptr - 1) == TCHAR('<'))
		{
			return true;
		}
	}
	else if(Ch == TCHAR('<') || Ch == TCHAR('>'))
	{
		return true;
	}

	return false;
}

/** Checks if the passed character is an operator */
static bool IsPartOfTagOperator(TCHAR Char)
{
	if(Char == TCHAR('<') ||
		Char == TCHAR('/') ||
		Char == TCHAR('>'))
	{
		return true;
	}
	return false;
}

/** Checks if the passed character is a quote */
static bool IsQuote(TCHAR Char)
{
	return Char == TCHAR('\"');
}

void FXmlFile::Tokenize(FStringView Input, TArray<FString>& Tokens)
{
	FString WorkingToken;
	ON_SCOPE_EXIT
	{
		// Add working token if it still exists
		if (WorkingToken.Len())
		{
			Tokens.Add(MoveTemp(WorkingToken));
		}
	};

	enum TOKENTYPE { OPERATOR, STRING } Type = STRING;
	bool bInQuote = false;

	const TCHAR* PtrStart = GetData(Input);
	const TCHAR* PtrEnd   = PtrStart + GetNum(Input);
	for (const TCHAR* Ptr = PtrStart; Ptr != PtrEnd; ++Ptr)
	{
		TCHAR Ch = *Ptr;

		if(IsWhiteSpace(Ch) && !bInQuote 
			&& (!bHackAttemptToPreserveWhitespace || Ch != TEXT('\n')))
		{
			// End the current token 
			if(WorkingToken.Len())
			{
				Tokens.Add(MoveTemp(WorkingToken));
				checkSlow(WorkingToken.Len() == 0);
			}

			continue;
		}

		// Mark the start of a token
		if(!WorkingToken.Len())
		{
			WorkingToken += Ch;
			if(CheckTagOperator(PtrStart, Ptr, PtrEnd))
			{
				// Add the working token if it's final (ie: ends with '>')
				if (Ch == TCHAR('>'))
				{
					Tokens.Add(MoveTemp(WorkingToken));
					checkSlow(WorkingToken.Len() == 0);
				}
				else
				{
					Type = OPERATOR;
				}
			}
			else
			{
				Type = STRING;
			}

			continue;
		}

		// Already in a token, so continue parsing
		if(Type == OPERATOR)
		{
			// Still the tag, so add it to the working token
			if(CheckTagOperator(PtrStart, Ptr, PtrEnd))
			{
				WorkingToken += Ch;

				// Add the working token if it's final (ie: ends with '>')
				if (Ch == TCHAR('>'))
				{
					Tokens.Add(MoveTemp(WorkingToken));
					checkSlow(WorkingToken.Len() == 0);
				}
			}

			// Not a tag operator anymore, so add the old token and start a new one
			else
			{
				Tokens.Add(MoveTemp(WorkingToken));
				checkSlow(WorkingToken.Len() == 0);
				WorkingToken += Ch;
				Type = STRING;
			}
		}
		else // STRING
		{
			if (IsQuote(Ch))
			{
				bInQuote = !bInQuote;
			}

			// Still a string. Allow '>' within a string
			if(!CheckTagOperator(PtrStart, Ptr, PtrEnd) || (bInQuote && Ch == TCHAR('>')))
			{
				WorkingToken += Ch;
			}

			// Moving back to operator
			else
			{
				Tokens.Add(MoveTemp(WorkingToken));
				checkSlow(WorkingToken.Len() == 0);
				WorkingToken += Ch;
				bInQuote = false;

				// Add the working token if it's final (ie: ends with '>')
				if (Ch == TCHAR('>'))
				{
					Tokens.Add(MoveTemp(WorkingToken));
					checkSlow(WorkingToken.Len() == 0);
				}
				else
				{
					Type = OPERATOR;
				}
			}
		}
	}
}

TArray<FString> FXmlFile::Tokenize(const TArray<FString>& Input)
{
	TArray<FString> Tokens;
	Tokens.Reserve(Input.Num());
	for(const FString& Line : Input)
	{
		Tokenize(Line, Tokens);
	}
	return Tokens;
}

static void AddAttribute(const FString& InToken, TArray<FXmlAttribute>& OutAttributes)
{
	int32 EqualsIdx;
	if(InToken.FindChar(TEXT('='), EqualsIdx))
	{
		bool bQuotesRemoved = false;
		FString Value = InToken.Mid(EqualsIdx + 1);
		Value.TrimQuotesInline(&bQuotesRemoved);

		// The xml spec allows single and double quotes, so this is a valid case to support: https://www.w3.org/TR/xml/#NT-AttValue
		if (!bQuotesRemoved)
		{
			Value.TrimCharInline('\'', &bQuotesRemoved);
		}

		if(bQuotesRemoved)
		{
			int32 AmpIdx;
			if(Value.FindChar(TEXT('&'), AmpIdx))
			{
				Value.ReplaceInline(TEXT("&quot;"), TEXT("\""), ESearchCase::CaseSensitive);
				Value.ReplaceInline(TEXT("&amp;"), TEXT("&"), ESearchCase::CaseSensitive);
				Value.ReplaceInline(TEXT("&apos;"), TEXT("'"), ESearchCase::CaseSensitive);
				Value.ReplaceInline(TEXT("&lt;"), TEXT("<"), ESearchCase::CaseSensitive);
				Value.ReplaceInline(TEXT("&gt;"), TEXT(">"), ESearchCase::CaseSensitive);
			}
			OutAttributes.Add(FXmlAttribute(InToken.Left(EqualsIdx), Value));
		}
	}
}

enum class ETryParseTagResult
{
	Error,
	None,
	ParsedStartTag,         // <key>
	ParsedEmptyElementTag,  // <key />
	ParsedEndTag,           // </key>
	NotATag                 // content - iterator is not advanced past this
};

static ETryParseTagResult TryParseTag(FString& OutTag, TArray<FXmlAttribute>& Attributes, const FString*& InOutToken, const FString* LastToken)
{
	const FString* Token = InOutToken;

	if (Token == LastToken)
	{
		return ETryParseTagResult::None;
	}

	if (*Token == TEXT(">") || *Token == TEXT("/>"))
	{
		// Error: Found an unexpected end-of-tag (ex: >list>)
		return ETryParseTagResult::Error;
	}

	// Found an end tag
	if (*Token == TEXT("</"))
	{
		++Token;

		if (Token == LastToken || *Token == TEXT("<") || *Token == TEXT("</") || *Token == TEXT(">") || *Token == TEXT("/>"))
		{
			// Error: Didn't find a tag name before the end of the tokens (ex: </>)
			return ETryParseTagResult::Error;
		}

		OutTag = *Token++;

		for (;;)
		{
			if (Token == LastToken || *Token == TEXT("<") || *Token == TEXT("</") || *Token == TEXT("/>"))
			{
				// Error: Didn't find a closing bracket (ex: </EndToken<)
				return ETryParseTagResult::Error;
			}

			if (*Token++ == TEXT(">"))
			{
				InOutToken = Token;
				return ETryParseTagResult::ParsedEndTag;
			}

			// Don't error if we find text after the final tag - they may be like options or modifiers or some such
		}
	}

	if (*Token != TEXT("<"))
	{
		// Found a non-tag
		// Don't advance the Token iterator because we want the caller to be able to examine it
		InOutToken = Token;
		return ETryParseTagResult::NotATag;
	}

	++Token;

	if (Token == LastToken || *Token == TEXT("<") || *Token == TEXT("</") || *Token == TEXT(">") || *Token == TEXT("/>"))
	{
		// Error: Didn't find a tag name before the end of the tokens (ex: <)
		return ETryParseTagResult::Error;
	}

	// Copy tag to output
	OutTag = *Token++;

	for (;;)
	{
		if (Token == LastToken)
		{
			// Error: Didn't find the closing tag (ex: <key)
			return ETryParseTagResult::Error;
		}

		const FString& TokenStr = *Token++;

		// Closing the tag and also finalizing it
		if (TokenStr == TEXT("/>"))
		{
			InOutToken = Token;
			return ETryParseTagResult::ParsedEmptyElementTag;
		}

		// Closing the tag starting
		if (TokenStr == TEXT(">"))
		{
			InOutToken = Token;
			return ETryParseTagResult::ParsedStartTag;
		}

		// Error
		if (TokenStr == TEXT("<") || TokenStr == TEXT("</"))
		{
			// Error: malformed file (ex: <key<)
			return ETryParseTagResult::Error;
		}

		// Assume this is an attribute
		AddAttribute(TokenStr, Attributes);
	}
}

FXmlNode* FXmlFile::CreateRootNode(TArrayView<const FString> Tokens)
{
	// Algorithm (Draft):
	//  - First found token should always be a '<'
	//  - Extract tag
	//  - Check next token
	//    - If '<', recursively deepen
	//    - If string token, parse until '<' found to recursively deepen
	//  - Continue parsing until </tag> for self is found
	//  - Return own constructed node (and index of next starting point

	// Stack of nodes currently being parsed - Last() is the current node being parsed
	TArray<FXmlNode*> ParsingNodeStack;
	ON_SCOPE_EXIT
	{
		// Clean up partially-parsed nodes - only the root node needs deletion because nested nodes
		// will have been added to the Children list of that node, and Delete already cleans those up.
		if (ParsingNodeStack.Num() != 0)
		{
			delete ParsingNodeStack[0];
		}
	};

	const FString* Token     = Tokens.GetData();
	const FString* LastToken = Token + Tokens.Num();
	for (;;)
	{
		FString               Tag;
		TArray<FXmlAttribute> Attributes;

		ETryParseTagResult ParseResult = TryParseTag(Tag, Attributes, Token, LastToken);
		if (ParseResult == ETryParseTagResult::None)
		{
			// Run out of tokens mid-parse

			if (!ParsingNodeStack.IsEmpty())
			{
				bCreationFailed = true;
				ErrorMessage = NSLOCTEXT("XmlParser", "MalformedXMLFile", "Malformed Xml File").ToString();
				return nullptr;
			}

			// Should only be possible if there was no content - original parser
			// would return nullptr without setting bCreationFailed and ErrorMessage
			return nullptr;
		}

		if (ParseResult == ETryParseTagResult::Error)
		{
			// Malformed tag

			bCreationFailed = true;
			ErrorMessage = NSLOCTEXT("XmlParser", "MalformedXMLFile", "Malformed Xml File").ToString();
			return nullptr;
		}

		if (ParseResult == ETryParseTagResult::ParsedEndTag)
		{
			// Found an end tag - pop the current node and return to parsing the parent

			if (ParsingNodeStack.IsEmpty())
			{
				// Error: encountered an end tag before we found any opening tag
				bCreationFailed = true;
				ErrorMessage = NSLOCTEXT("XmlParser", "MalformedXMLFile", "Malformed Xml File").ToString();
				return nullptr;
			}

			if (ParsingNodeStack.Last()->Tag != Tag)
			{
				// Error: Tag and ending tag don't match
				bCreationFailed = true;
				ErrorMessage = NSLOCTEXT("XmlParser", "MalformedXMLFile", "Malformed Xml File").ToString();
				return nullptr;
			}

			FXmlNode* FullyParsedNode = ParsingNodeStack.Pop();

			if (ParsingNodeStack.IsEmpty())
			{
				// Found the end tag of the root node - stop
				return FullyParsedNode;
			}

			continue;
		}

		if (ParseResult == ETryParseTagResult::NotATag)
		{
			// Found some content - add it to the content of the current node

			if (ParsingNodeStack.IsEmpty())
			{
				if (bHackAttemptToPreserveWhitespace && Algo::AllOf(*Token, IsWhiteSpace))
				{
					// Whitespace is allowed outside the root node.
					++Token;
					continue;
				}

				// Error: encountered content tokens outside of a nested tag
				bCreationFailed = true;
				ErrorMessage = NSLOCTEXT("XmlParser", "MalformedXMLFile", "Malformed Xml File").ToString();
				return nullptr;
			}

			FString& Content = ParsingNodeStack.Last()->Content;

			if (Content.Len() > 0 
				&& (!bHackAttemptToPreserveWhitespace || !Content.EndsWith(TEXT("\n"))))
			{
				Content += TEXT(" ");
			}
			Content += *Token++;

			continue;
		}

		// Found a nested node

		check(ParseResult == ETryParseTagResult::ParsedStartTag || ParseResult == ETryParseTagResult::ParsedEmptyElementTag);

		FXmlNode* NewNode = new FXmlNode();
		NewNode->Tag = MoveTemp(Tag);
		NewNode->Attributes = MoveTemp(Attributes);

		// If we are already parsing nodes, this is a child node, so add it to the parent
		if (!ParsingNodeStack.IsEmpty())
		{
			ParsingNodeStack.Last()->Children.Add(NewNode);
		}
		else if (ParseResult == ETryParseTagResult::ParsedEmptyElementTag)
		{
			// If we just parsed an empty element root node, stop
			return NewNode;
		}

		if (ParseResult == ETryParseTagResult::ParsedStartTag)
		{
			ParsingNodeStack.Add(NewNode);
		}
	}
}

void FXmlFile::HookUpNextPtrs(FXmlNode* Node)
{
	if(Node == nullptr)
	{
		return;
	}

	for(int32 i = 0; i < Node->Children.Num(); ++i)
	{
		HookUpNextPtrs(Node->Children[i]);

		if(i != Node->Children.Num() - 1)
		{
			Node->Children[i]->NextNode = Node->Children[i + 1];
		}
	}
}

void FXmlFile::CreateNodes(const TArray<FString>& Tokens)
{
	// Assumption..There is only 1 top-level node which contains everything inside of it
	bCreationFailed = false;
	FXmlNode* Root = CreateRootNode(Tokens);

	if(Root)
	{
		bFileLoaded = true;

		// Hook up next ptrs
		HookUpNextPtrs(Root);

		// Save it
		RootNode = Root;
	}
	else
	{
		bFileLoaded = false;
		ErrorMessage = NSLOCTEXT("XmlParser", "NodeCreateFail", "Failed to parse the loaded document").ToString();
	}
}

void FXmlFile::WriteNodeHierarchy(const FXmlNode& Node, const FString& Indent, FString& Output)
{
	// Write the tag
	Output += Indent + FString::Printf(TEXT("<%s"), *Node.GetTag());
	for(const FXmlAttribute& Attribute: Node.GetAttributes())
	{
		FString EscapedValue = Attribute.GetValue();
		EscapedValue.ReplaceInline(TEXT("&"), TEXT("&amp;"), ESearchCase::CaseSensitive);
		EscapedValue.ReplaceInline(TEXT("\""), TEXT("&quot;"), ESearchCase::CaseSensitive);
		EscapedValue.ReplaceInline(TEXT("'"), TEXT("&apos;"), ESearchCase::CaseSensitive);
		EscapedValue.ReplaceInline(TEXT("<"), TEXT("&lt;"), ESearchCase::CaseSensitive);
		EscapedValue.ReplaceInline(TEXT(">"), TEXT("&gt;"), ESearchCase::CaseSensitive);
		Output += FString::Printf(TEXT(" %s=\"%s\""), *Attribute.GetTag(), *EscapedValue);
	}

	// Write the node contents
	const FXmlNode* FirstChildNode = Node.GetFirstChildNode();
	if(FirstChildNode == nullptr)
	{
		const FString& Content = Node.GetContent();
		if(Content.Len() == 0)
		{
			Output += TEXT(" />") LINE_TERMINATOR;
		}
		else
		{
			Output += TEXT(">") + Content + FString::Printf(TEXT("</%s>"), *Node.GetTag()) + LINE_TERMINATOR;
		}
	}
	else
	{
		Output += TEXT(">") LINE_TERMINATOR;
		for(const FXmlNode* ChildNode = FirstChildNode; ChildNode != nullptr; ChildNode = ChildNode->GetNextNode())
		{
			WriteNodeHierarchy(*ChildNode, Indent + TEXT("\t"), Output);
		}
		Output += Indent + FString::Printf(TEXT("</%s>"), *Node.GetTag()) + LINE_TERMINATOR;
	}
}

void FXmlFile::EnableAttemptToPreserveWhitespaceHack()
{
	bHackAttemptToPreserveWhitespace = true;
}
