// Copyright Epic Games, Inc. All Rights Reserved.

#include "XmlFile.h"
#include "Misc/AutomationTest.h"
BEGIN_DEFINE_SPEC(FXmlFileSpec, "System.Core.XmlFile", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ProductFilter)

END_DEFINE_SPEC(FXmlFileSpec)

void FXmlFileSpec::Define()
{
	Describe("FXmlFileSpec", [this]()
	{
		It("Should return no root node for an empty file", [this]()
		{
			FXmlFile XmlFile(FString(), EConstructMethod::ConstructFromBuffer);
			FXmlNode* RootNode = XmlFile.GetRootNode();
			TestNull("RootNode", RootNode);
		});
		
		It("Should return error if the file does not contain a root node", [this]()
		{
			FXmlFile XmlFile(FString("SomeTextInHere"), EConstructMethod::ConstructFromBuffer);
			FXmlNode* RootNode = XmlFile.GetRootNode();
			TestNull("RootNode", RootNode);
			TestFalse(TEXT("XmlFile.IsValid()"), XmlFile.IsValid());
			TestEqual(TEXT("last error"), *XmlFile.GetLastError(), TEXT("Failed to parse the loaded document"));
		});

		It("Should parse single and double quoted attributes", [this]()
		{
			/* The xml included here is technically invalid because attribute d and attribute e have quotes in them that
			* should make their value terminate early and the xml invalid, but I wanted to show the new behaviour with
			* single quotes matches the existing (incorrect) behaviour with double quotes.
			*/
			FXmlFile XmlFile(FString("<test a='aTest' b=\"bTest\" c=\"c'sTest\" d='d'sTest' e=\"e\"Test\"/>"), EConstructMethod::ConstructFromBuffer);
			FXmlNode* RootNode = XmlFile.GetRootNode();
			TestNotNull("RootNode", RootNode);
			if (RootNode != nullptr)
			{
				TestEqual("RootNode->GetAttributes().Num()", RootNode->GetAttributes().Num(), 5);
				const FString aValue = RootNode->GetAttribute(FString("a"));
				const FString bValue = RootNode->GetAttribute(FString("b"));
				const FString cValue = RootNode->GetAttribute(FString("c"));
				const FString dValue = RootNode->GetAttribute(FString("d"));
				const FString eValue = RootNode->GetAttribute(FString("e"));
				TestFalse("aValue.IsEmpty()", aValue.IsEmpty());
				TestEqual("aValue", aValue, FString("aTest"));
				TestFalse("bValue.IsEmpty()", bValue.IsEmpty());
				TestEqual("bValue", bValue, FString("bTest"));
				TestFalse("cValue.IsEmpty()", cValue.IsEmpty());
				TestEqual("cValue", cValue, FString("c'sTest"));
				TestFalse("dValue.IsEmpty()", dValue.IsEmpty());
				TestEqual("dValue", dValue, FString("d'sTest"));
				TestFalse("eValue.IsEmpty()", eValue.IsEmpty());
				TestEqual("eValue", eValue, FString("e\"Test"));
			}
		});
	
		It("Should parse nested text and nodes inside a node", [this]()
		{
			FXmlFile XmlFile(FString("<root><container>Some Text<child></child></container></root>"), EConstructMethod::ConstructFromBuffer);
			FXmlNode* RootNode = XmlFile.GetRootNode();
			TestNotNull("RootNode", RootNode);
			TestTrue(TEXT("XmlFile.IsValid()"), XmlFile.IsValid());

			const FXmlNode* Container = RootNode->GetFirstChildNode();
			TestEqual(TEXT("tag"), Container->GetTag(), TEXT("container"));

			TestEqual(TEXT("Content"), Container->GetContent(), TEXT("Some Text"));
			const FXmlNode* Child = Container->GetFirstChildNode();
			TestEqual(TEXT("tag"), Child->GetTag(), TEXT("child"));
		});
	});
};