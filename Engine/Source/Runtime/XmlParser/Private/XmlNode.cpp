// Copyright Epic Games, Inc. All Rights Reserved.

#include "XmlNode.h"


const FString& FXmlAttribute::GetTag() const
{
	return Tag;
}

const FString& FXmlAttribute::GetValue() const
{
	return Value;
}

void FXmlNode::Delete()
{
	TArray<FXmlNode*> ToDelete = MoveTemp(Children);
	check(Children.IsEmpty());

	for (int32 Index = 0; Index != ToDelete.Num(); ++Index)
	{
		FXmlNode* NodeToDelete = ToDelete[Index];
		ToDelete.Append(MoveTemp(NodeToDelete->Children));
		check(NodeToDelete->Children.IsEmpty());
	}

	for (FXmlNode* Node : ToDelete)
	{
		delete Node;
	}
}

const FXmlNode* FXmlNode::GetNextNode() const
{
	return NextNode;
}

const TArray<FXmlNode*>& FXmlNode::GetChildrenNodes() const
{
	return Children;
}

const FXmlNode* FXmlNode::GetFirstChildNode() const
{
	if(Children.Num() > 0)
	{
		return Children[0];
	}
	else
	{
		return nullptr;
	}
}

const FXmlNode* FXmlNode::FindChildNode(const FString& InTag) const
{
	const int32 ChildCount = Children.Num();
	for(int32 ChildIndex = 0; ChildIndex < ChildCount; ++ChildIndex)
	{
		if(Children[ChildIndex] != nullptr && Children[ChildIndex]->GetTag() == InTag)
		{
			return Children[ChildIndex];
		}
	}

	return nullptr;
}

FXmlNode* FXmlNode::FindChildNode(const FString& InTag)
{
	return const_cast<FXmlNode*>(AsConst(*this).FindChildNode(InTag));
}

const FString& FXmlNode::GetTag() const
{
	return Tag;
}

const FString& FXmlNode::GetContent() const
{
	return Content;
}

void FXmlNode::SetContent( const FString& InContent )
{
	Content = InContent;
}

void FXmlNode::SetAttributes(const TArray<FXmlAttribute>& InAttributes)
{
	Attributes = InAttributes;
}

FString FXmlNode::GetAttribute(const FString& InTag) const
{
	for(auto Iter(Attributes.CreateConstIterator()); Iter; Iter++)
	{
		if(Iter->GetTag() == InTag)
		{
			return Iter->GetValue();
		}
	}
	return FString();
}

void FXmlNode::AppendChildNode(const FString& InTag, const FString& InContent, const TArray<FXmlAttribute>& InAttributes)
{
	auto NewNode = new FXmlNode;
	NewNode->Tag = InTag;
	NewNode->Content = InContent;
	NewNode->Attributes = InAttributes;

	auto NumChildren = Children.Num();
	if (NumChildren != 0)
	{
		Children[NumChildren - 1]->NextNode = NewNode;
	}
	Children.Push(NewNode);
}
