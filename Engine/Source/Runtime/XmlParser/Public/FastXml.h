// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/Platform.h"

class FText;

/**
 * Fast XML parser based on John W. Ratcliff's "FastXml" (see FastXml.tps and http://codesuppository.blogspot.com/2009/02/fastxml-extremely-lightweight-stream.html)
 * 
 * This is a simple XML parser that can load XML files very quickly.  The class has a single static method, ParseXmlFile() for
 * loading XML files.  The parser is designed to load files efficiently, but may not support all XML features or be resilient 
 * to malformed data.  Unlike the FXmlFile class, FFastXml does not generate an XML node tree.  Instead, you must supply a 
 * callback class using IFastXmlCallback and respond to elements and attributes as they are parsed.  Call the ParseXmlFile() 
 * function and pass in either the full path to the XML file to parse (XmlFilePath parameter), or load the file yourself and 
 * pass the contents using the XmlFileContents parameter.  One of either XmlFilePath or XmlFileContents must be valid for 
 * ParseXmlFile() to work.
 * 
 * Remember to add a module dependency on "XmlParser" in order to be able to call FFastXml::ParserXmlFile().
 */
class FFastXml
{

public:

	/**
	 * Quickly parse an XML file.  Pass in your implementation of the IFastXmlCallback interface with code to handle parsed 
	 * elements and attributes, along with either the full path to the XML file to load, or a string with the full XML file content.
	 * 
	 * @param	Callback			As the parser encounters XML elements or attributes, methods on this callback object will be called
	 * @param	XmlFilePath			The path on disk to the XML file to load, or an empty string if you'll be passing the XML file content in directly using the XmlFileContents parameter.
	 * @param	XmlFileContents		The full contents of the file to parse, or an empty string if you've passed a full path to the file to parse using the XmlFilePath parameter.  Note that this string will be modified during the parsing process, so make a copy of it first if you need it for something else (uncommon.)
	 * @param	FeedbackContext		Optional feedback context for reporting warnings or progress (GWarn is typically passed in.)  You can pass nullptr if you don't want any progress reported.
	 * @param	bShowSlowTaskDialog	True if we should display a 'please wait' dialog while we parse the file, if the feedback context supports that
	 * @param	bShowCancelButton	Whether the user is allowed to cancel the load of this XML file while in progress.  If the user cancels loading, an appropriate error message will be returned.  Only applies when bShowSlowTaskDialog is also enabled, and for feedback contexts that support cancel buttons.
	 * @param	OutErrorMessage		If anything went wrong or the user canceled the load, this error will contain a description of the problem
	 * @param	OutErrorLineNumber	Line number that any error happened on
	 *
	 * @return	Returns true if loading was successful, or false if anything went wrong or the user canceled.  When false is returned, an error message string will be set in the OutErrorMessage variable, along with the line number that parsing failed at in OutErrorLineNumber.
	 */
	XMLPARSER_API static bool ParseXmlFile( class IFastXmlCallback* Callback, const TCHAR* XmlFilePath, TCHAR* XmlFileContents, class FFeedbackContext* FeedbackContext, const bool bShowSlowTaskDialog, const bool bShowCancelButton, FText& OutErrorMessage, int32& OutErrorLineNumber );
	
};
	

/**
 * Create your own class that implements the IFastXmlCallback interface to process the XML elements as they
 * are loaded by FFastXml::ParseXmlFile().  You'll receive a ProcessElement() call for every XML element
 * that is encountered, along with a corresponding ProcessClose() when that element's scope has ended.
 * ProcessAttribute() will be called for any attributes found within the scope of the current element.
 */
class IFastXmlCallback
{

public:

	/**
	 * Called after the XML's header is parsed.  This is usually the first call that you'll get back.
	 *
	 * @param	ElementData			Optional data for this element, nullptr if none
	 * @param	XmlFileLineNumber	Line number in the XML file we're on
	 *
	 * @return	You should return true to continue processing the file, or false to stop processing immediately.
	 */
	virtual bool ProcessXmlDeclaration( const TCHAR* ElementData, int32 XmlFileLineNumber ) = 0;

	/**
	 * Called when a new XML element is encountered, starting a new scope.  You'll receive a call to ProcessClose()
	 * when this element's scope has ended.
	 *
	 * @param	ElementName			The name of the element
	 * @param	ElementData			Optional data for this element, nullptr if none
	 * @param	XmlFileLineNumber	The line number in the XML file we're on
	 *
	 * @return	You should return true to continue processing the file, or false to stop processing immediately.
	 */
	virtual bool ProcessElement( const TCHAR* ElementName, const TCHAR* ElementData, int32 XmlFileLineNumber ) = 0;
	
	/**
	 * Called when an XML attribute is encountered for the current scope's element.
	 *
	 * @param	AttributeName	The name of the attribute
	 * @param	AttributeValue	The value of the attribute
	 *
	 * @return	You should return true to continue processing the file, or false to stop processing immediately.
	 */
	virtual bool ProcessAttribute( const TCHAR* AttributeName, const TCHAR* AttributeValue ) = 0;

	/**
	 * Called when an element's scope ends in the XML file
	 *
	 * @param	ElementName		Name of the element whose scope closed
	 *
	 * @return	You should return true to continue processing the file, or false to stop processing immediately.
	 */
	virtual bool ProcessClose( const TCHAR* Element ) = 0;
	
	/**
	 * Called when a comment is encountered.  This can happen pretty much anywhere in the file.
	 *
	 * @param	Comment		The comment text
	 */
	virtual bool ProcessComment( const TCHAR* Comment ) = 0;
	
};


