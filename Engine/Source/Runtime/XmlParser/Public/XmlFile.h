// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/StringFwd.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "XmlNode.h"

class FXmlAttribute;
class FXmlNode;

namespace EConstructMethod
{
	enum Type
	{
		ConstructFromFile,
		ConstructFromBuffer,
	};
}

/** Class representing an XML file */
class FXmlFile
{
public:

	/** Constructs the file without a path */
	FXmlFile() : RootNode(nullptr), bFileLoaded(false) {}
	/** 
	 * Constructs the file with the passed path. InFile is either treated as a filename to open, or as a text
	 * buffer to load.
	 * @param	InFile				The path/text to use
	 * @param	ConstructMethod		Whether to load a file of use the string as a buffer of xml data
	 */
	XMLPARSER_API FXmlFile(const FString& InFile, EConstructMethod::Type ConstructMethod = EConstructMethod::ConstructFromFile);
	~FXmlFile() { Clear(); };

	FXmlFile(const FXmlFile& rhs) = delete;
	FXmlFile& operator=(const FXmlFile& rhs) = delete;

	/** 
	 * Loads the file with the passed path. Path is either treated as a filename to open, or as a text
	 * buffer to load.
	 * @param	Path				The path/text to use
	 * @param	ConstructMethod		Whether to load a file of use the string as a buffer of xml data
	 */
	XMLPARSER_API bool LoadFile(const FString& Path, EConstructMethod::Type ConstructMethod = EConstructMethod::ConstructFromFile);
	/** Gets the last error message from the class */
	XMLPARSER_API FString GetLastError() const;
	/** Clears the file of all internals. Note: Makes any existing pointers to FXmlNode's INVALID */
	XMLPARSER_API void Clear();
	/** Checks to see if a file is loaded */
	XMLPARSER_API bool IsValid() const;

	/** 
	 * Returns the root node of the loaded file. nullptr if no file loaded. 
	 * It is assumed that there will always be one and only one root node.
	 * @return						Pointer to root node
	 */
	XMLPARSER_API const FXmlNode* GetRootNode() const;

	/** 
	 * Returns the root node of the loaded file. nullptr if no file loaded. 
	 * It is assumed that there will always be one and only one root node.
	 * @return						Pointer to root node
	 */
	XMLPARSER_API FXmlNode* GetRootNode();

	/**
	 * Write to disk, UTF-16 format only at the moment
	 * @param	Path				File path to save to
	 * @return						Whether writing the XML to a file succeeded
	 */
	XMLPARSER_API bool Save(const FString& Path);

	/**
	 * Turns on a specific hack that is only used by the crash report client in order
	 * to retain newlines for callstacks. This is important on the server side when we 
	 * display the callstacks to content creators. Without the content-whitespace,
	 * they're bascially illegible. Neither version of the DOM we create is technically
	 * true to the XML spec.
	 */
	XMLPARSER_API void EnableAttemptToPreserveWhitespaceHack();

private:

	/** Pre-processes the file doing anything necessary (such as removing comments) */
	XMLPARSER_API void PreProcessInput(TArray<FString>& Input);
	/** Whites of the text at the specified locations in a passed-in array */
	XMLPARSER_API void WhiteOut(TArray<FString>& Input, int32 LineStart, int32 LineEnd, int32 IndexStart, int32 IndexEnd);
	/** Converts the passed input into a list of tokens for parsing */
	XMLPARSER_API void Tokenize(FStringView Input, TArray<FString>& Tokens);
	/** Converts the passed input into a list of tokens for parsing */
	XMLPARSER_API TArray<FString> Tokenize(const TArray<FString>& Input);
	/** 
	 * Add an attribute to the passed-in array.
	 * This makes the assumption that an attribute comes in as one 'token' (i.e. no spaces between tag="value").
	 */
	static XMLPARSER_API void AddAttribute(const FString& InToken, TArray<FXmlAttribute>& OutAttributes);
	/** Recursively creates nodes from the list of tokens */
	XMLPARSER_API FXmlNode* CreateRootNode(TArrayView<const FString> Tokens);
	/** Hook next ptrs up recursively */
	XMLPARSER_API void HookUpNextPtrs(FXmlNode* Node);
	/** Creates the internal file representation as a bunch of FXmlNode's */
	XMLPARSER_API void CreateNodes(const TArray<FString>& Tokens);
	/** Writes a node hierarchy at the given root to a string */
	static XMLPARSER_API void WriteNodeHierarchy(const FXmlNode& Node, const FString& Indent, FString& Output);

private:

	/** The passed-in path of the loaded file (might be absolute or relative) */
	FString LoadedFile;
	/** An error message generated on errors to return to the client */
	FString ErrorMessage;
	/** A pointer to the root node */
	FXmlNode* RootNode;
	/** Flag for whether a file is loaded or not */
	bool bFileLoaded;
	/** Flag for whether the node creation process failed or not */
	bool bCreationFailed;

	/** Flag for the whitespace hack used in the crash report client */
	bool bHackAttemptToPreserveWhitespace = false;
};
