// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"

/** Forward Decl for implementation */
class FXmlFile;

class FXmlAttribute
{
public:
	FXmlAttribute(const FString& InTag, const FString& InValue)
		: Tag(InTag)
		, Value(InValue)
	{
	}

	/** Gets the tag of the attribute */
	XMLPARSER_API const FString& GetTag() const;

	/** Gets the value of the attribute */
	XMLPARSER_API const FString& GetValue() const;

private:
	/** The tag string */
	FString Tag;

	/** The value string */
	FString Value;
};

/** Xml Node representing a line in an xml file */
class FXmlNode
{
	friend class FXmlFile;

private:

	/** Default ctor, private for FXmlFile use only */
	FXmlNode() : NextNode(nullptr) {}
	/** No copy ctor allowed */
	FXmlNode(const FXmlNode& rhs) {}
	/** dtor */
	~FXmlNode() { Delete(); }

private:

	/** Recursively deletes the nodes for cleanup */
	XMLPARSER_API void Delete();

public:

	/** Gets the next node in a list of nodes */
	XMLPARSER_API const FXmlNode* GetNextNode() const;
	/** Gets a list of children nodes */
	XMLPARSER_API const TArray<FXmlNode*>& GetChildrenNodes() const;
	/** Gets the first child of this node which can be iterated into with GetNextNode */
	XMLPARSER_API const FXmlNode* GetFirstChildNode() const;
	/** Finds the first child node that contains the specified tag */
	XMLPARSER_API const FXmlNode* FindChildNode(const FString& InTag) const;
	/** Finds the first child node that contains the specified tag */
	XMLPARSER_API FXmlNode* FindChildNode(const FString& InTag);
	/** Gets the tag of the node */
	XMLPARSER_API const FString& GetTag() const;
	/** Gets the value of the node */
	XMLPARSER_API const FString& GetContent() const;
	/** Sets the new value of the node */
	XMLPARSER_API void SetContent(const FString& InContent);
	/** Sets the attributes of the node */
	XMLPARSER_API void SetAttributes(const TArray<FXmlAttribute>& InAttributes);

	/**
	 * Gets all of the attributes in this node
	 *
	 * @return	List of attributes in this node
	 */
	const TArray<FXmlAttribute>& GetAttributes() const
	{
		return Attributes;
	}

	/** Gets an attribute that corresponds with the passed-in tag */
	XMLPARSER_API FString GetAttribute(const FString& InTag) const;

	/**
	 *  Adds a child node to this node  
	 * @param	InTag				The tag of the child node
	 * @param	InContent			(optional) The content of the child node
	 * @param	InAttributes		(optional) An array of attributes of the child node
	 */
	XMLPARSER_API void AppendChildNode(const FString& InTag, const FString& InContent = FString(), const TArray<FXmlAttribute>& InAttributes = TArray<FXmlAttribute>());
private:

	/** The list of children nodes */
	TArray<FXmlNode*> Children;
	/** Attributes of this node */
	TArray<FXmlAttribute> Attributes;
	/** Tag of the node */
	FString Tag;
	/** Content of the node */
	FString Content;
	/** Next pointer */
	FXmlNode* NextNode;

};
