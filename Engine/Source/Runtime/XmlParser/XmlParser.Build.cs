// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class XmlParser : ModuleRules
{
	public XmlParser( ReadOnlyTargetRules Target ) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] 
			{ 
				"Core",
			});
		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;

		// PCH is disabled here otherwise the module will use Core's shared PCH and fail to compile due
		// to a clash with UE_DEFINE_LEGACY_MATH_CONSTANT_MACRO_NAMES being defined in UnrealMathUtility.h.
		PCHUsage = PCHUsageMode.NoPCHs;
		PrivateDefinitions.Add("UE_DEFINE_LEGACY_MATH_CONSTANT_MACRO_NAMES=0");
	}
}
